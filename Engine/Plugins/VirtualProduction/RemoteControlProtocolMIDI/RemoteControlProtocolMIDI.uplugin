{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Protocol MIDI",
	"Description": "Allows interactions between MIDI and RemoteControl API.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RemoteControlProtocolMIDI",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac"
			]
		},
		{
			"Name": "RemoteControlProtocolMIDIEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "MIDIDevice",
			"Enabled": true
		}
	]
}