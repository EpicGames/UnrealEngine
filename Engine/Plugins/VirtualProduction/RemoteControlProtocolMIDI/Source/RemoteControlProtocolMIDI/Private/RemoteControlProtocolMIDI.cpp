// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolMIDI.h"

#include "MIDIDeviceManager.h"
#include "RemoteControlLogger.h"
#include "RemoteControlProtocolMIDIModule.h"
#include "RemoteControlProtocolMIDISettings.h"

#if WITH_EDITOR
#include "IRCProtocolBindingList.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#endif

#define LOCTEXT_NAMESPACE "FRemoteControlProtocolMIDI"

const FName FRemoteControlProtocolMIDI::ProtocolName = TEXT("MIDI");

#if WITH_EDITOR

namespace RemoteControlMIDIProtocolColumns
{
	static FName Channel = TEXT("Channel");
	static FName Identifier = TEXT("Identifier");
	static FName Type = TEXT("Type");
}

#endif // WITH_EDITOR

int32 FRemoteControlMIDIDevice::ResolveDeviceId(const TArray<FFoundMIDIDevice>& InFoundDevices)
{
	if(InFoundDevices.Num() == 0)
	{
		// This is only displayed because it's valid and a common use case that bindings would be setup but unused by this particular project participant 
		UE_LOGF(LogRemoteControlProtocolMIDI, Display, "RemoteControlProtocolMIDI bindings specified, but no devices are available.")
		return ResolvedDeviceId = INDEX_NONE;
	}
	
	// Call project settings if they're used
	if(DeviceSelector == ERemoteControlMIDIDeviceSelector::ProjectSettings)
	{
		URemoteControlProtocolMIDISettings* MIDISettings = GetMutableDefault<URemoteControlProtocolMIDISettings>();
		// Check that this isn't called from itself
		if(!ensureMsgf(MIDISettings->DefaultDevice != *this, TEXT("DeviceSelector == ERemoteControlMIDIDeviceSelector::ProjectSettings on RemoteControlProtocolMIDISettings.DefaultDevice! This is not valid.")))
		{
			return ResolvedDeviceId = INDEX_NONE;
		}


		return ResolvedDeviceId = MIDISettings->DefaultDevice.ResolveDeviceId(InFoundDevices);
	}

	// reset flag
	bDeviceIsAvailable = true;
	int32 FoundDeviceId = INDEX_NONE;
	// Convert for comparison
	const FString DeviceNameStr = DeviceName.ToString();
	for(const FFoundMIDIDevice& FoundDevice : InFoundDevices)
	{
		if(DeviceSelector == ERemoteControlMIDIDeviceSelector::DeviceId && DeviceId >= 0)
		{
			// User specified device id was found and valid, just return it
			if(FoundDevice.DeviceID == DeviceId)
			{
				if(FoundDevice.bIsAlreadyInUse)
				{
					bDeviceIsAvailable = false;
					FoundDeviceId = FoundDevice.DeviceID;
					break;
				}

				return ResolvedDeviceId = FoundDevice.DeviceID;
			}
		}

		if(DeviceSelector == ERemoteControlMIDIDeviceSelector::DeviceName && DeviceName != NAME_None)
		{
			if(FoundDevice.DeviceName.Equals(DeviceNameStr, ESearchCase::IgnoreCase))
			{
				// If the device is found, reset this DeviceId to the one found
				return ResolvedDeviceId = FoundDevice.DeviceID;
			}
		}
	}

	if(!bDeviceIsAvailable)
	{
		UE_LOGF(LogRemoteControlProtocolMIDI, Warning, "MIDI Device with Id %i was found, but already in use and unavailable.", FoundDeviceId);
		bDeviceIsAvailable = false;
		// Set the correct device id (so it's valid), but it won't be used.
		return ResolvedDeviceId = FoundDeviceId;
	}

	// If we're here, no matching device was found
	if(DeviceSelector == ERemoteControlMIDIDeviceSelector::DeviceId)
	{
		UE_LOGF(LogRemoteControlProtocolMIDI, Warning, "MIDI Device with user specified Id %i was not found, using Device 1.", DeviceId);
		// We've already checked if theres one or more devices
		return ResolvedDeviceId = InFoundDevices[0].DeviceID;
	}
	else
	{
		UE_LOGF(LogRemoteControlProtocolMIDI, Warning, "MIDI Device with specified name %ls was not found, using Device 1.", *DeviceName.ToString());
		// We've already checked if theres one or more devices
		return ResolvedDeviceId = InFoundDevices[0].DeviceID;
	}
}

void FRemoteControlMIDIDevice::SetDevice(const int32 InDeviceId, const FName& InDeviceName)
{
	DeviceSelector = ERemoteControlMIDIDeviceSelector::DeviceName;

	// Set's resolved id, not device id as that's for manual user specification
	ResolvedDeviceId = InDeviceId;
	DeviceName = InDeviceName;
}

void FRemoteControlMIDIDevice::SetUseProjectSettings()
{
	DeviceSelector = ERemoteControlMIDIDeviceSelector::ProjectSettings;
}

void FRemoteControlMIDIDevice::SetUserDeviceId()
{
	DeviceSelector = ERemoteControlMIDIDeviceSelector::DeviceId;
}

FText FRemoteControlMIDIDevice::ToDisplayName() const
{
	return FText::Format(LOCTEXT("DeviceMenuItem", "[{0}]: {1}"), FText::AsNumber(ResolvedDeviceId), FText::FromName(DeviceName));
}

bool FRemoteControlMIDIProtocolEntity::IsSame(const FRemoteControlProtocolEntity* InOther)
{
	if(const FRemoteControlMIDIProtocolEntity* Other = static_cast<const FRemoteControlMIDIProtocolEntity*>(InOther))
	{
		return Device.ResolvedDeviceId == Other->Device.ResolvedDeviceId
			&& EventType == Other->EventType
			&& Channel == Other->Channel
			&& MessageData1 == Other->MessageData1;	
	}

	return false;
}

#if WITH_EDITOR

void FRemoteControlMIDIProtocolEntity::RegisterProperties()
{
	EXPOSE_PROTOCOL_PROPERTY(RemoteControlMIDIProtocolColumns::Channel, FRemoteControlMIDIProtocolEntity, Channel);

	EXPOSE_PROTOCOL_PROPERTY(RemoteControlMIDIProtocolColumns::Identifier, FRemoteControlMIDIProtocolEntity, MessageData1);

	EXPOSE_PROTOCOL_PROPERTY(RemoteControlMIDIProtocolColumns::Type, FRemoteControlMIDIProtocolEntity, EventType);
}

#endif // WITH_EDITOR

void FRemoteControlProtocolMIDI::Bind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}

	FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = InRemoteControlProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();

	TArray<FFoundMIDIDevice> Devices;
	UMIDIDeviceManager::FindMIDIDevices(Devices);
	MIDIProtocolEntity->Device.ResolveDeviceId(Devices);
	
	const int32 MIDIDeviceId = MIDIProtocolEntity->Device.ResolvedDeviceId;
	ensureMsgf(MIDIDeviceId >= 0, TEXT("MIDI Resolved Device Id was -1, ensure ResolveDeviceId is called first!"));

	TStrongObjectPtr<UMIDIDeviceInputController>* MIDIDeviceInputControllerPtr = MIDIDevices.Find(MIDIDeviceId);
	if (MIDIDeviceInputControllerPtr == nullptr)
	{
		TStrongObjectPtr<UMIDIDeviceInputController> MIDIDeviceInputController = TStrongObjectPtr<UMIDIDeviceInputController>(UMIDIDeviceManager::CreateMIDIDeviceInputController(MIDIDeviceId));
		if (!MIDIDeviceInputController.IsValid())
		{
			// Impossible to create a midi input controller
			return;
		}
		MIDIDeviceInputControllerPtr = &MIDIDeviceInputController;

		MIDIDeviceInputController->OnMIDIRawEvent.AddSP(this, &FRemoteControlProtocolMIDI::OnReceiveEvent);
		MIDIDevices.Add(MIDIDeviceId, MIDIDeviceInputController);
	}

	if (MIDIProtocolEntity->EventType == EMIDIEventType::NoteOn)
	{
		MIDIDeviceBindings_NoteOn.Add(MIDIProtocolEntity->GetPropertyId(), InRemoteControlProtocolEntityPtr);
	}
	else if (MIDIProtocolEntity->EventType == EMIDIEventType::ChannelAfterTouch)
	{
		MIDIDeviceBindings_ChannelAfterTouch.Add(MIDIProtocolEntity->GetPropertyId(), InRemoteControlProtocolEntityPtr);
	}
	else if (MIDIProtocolEntity->EventType == EMIDIEventType::ControlChange)
	{
		TMap<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>* EntityBindingsMapPtr = MIDIDeviceBindings_ControlChange.Find(MIDIDeviceInputControllerPtr->Get());
		if (EntityBindingsMapPtr == nullptr)
		{
			EntityBindingsMapPtr = &MIDIDeviceBindings_ControlChange.Add(MIDIDeviceInputControllerPtr->Get());
		}

		if (TArray<FRemoteControlProtocolEntityWeakPtr>* ProtocolEntityBindingsArrayPtr = EntityBindingsMapPtr->Find(MIDIProtocolEntity->MessageData1))
		{
			ProtocolEntityBindingsArrayPtr->Emplace(MoveTemp(InRemoteControlProtocolEntityPtr));
		}
		else
		{
			TArray<FRemoteControlProtocolEntityWeakPtr> NewEntityBindingsArrayPtr { MoveTemp(InRemoteControlProtocolEntityPtr) };

			EntityBindingsMapPtr->Add(MIDIProtocolEntity->MessageData1, MoveTemp(NewEntityBindingsArrayPtr));
		}	
	}
}

void FRemoteControlProtocolMIDI::Unbind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr)
{
	if (!ensure(InRemoteControlProtocolEntityPtr.IsValid()))
	{
		return;
	}

	FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = InRemoteControlProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();

	if (MIDIProtocolEntity->EventType  == EMIDIEventType::NoteOn)
	{
		MIDIDeviceBindings_NoteOn.Remove(MIDIProtocolEntity->GetPropertyId());
	}
	else if (MIDIProtocolEntity->EventType == EMIDIEventType::ChannelAfterTouch)
	{
		MIDIDeviceBindings_ChannelAfterTouch.Remove(MIDIProtocolEntity->GetPropertyId());
	}
	else if (MIDIProtocolEntity->EventType == EMIDIEventType::ControlChange)
	{
		for (TPair<UMIDIDeviceInputController*, TMap<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>>& BindingsPair : MIDIDeviceBindings_ControlChange)
		{
			TMap<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>& BindingMap = BindingsPair.Value;

			for (TPair<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>& BindingMapPair : BindingMap)
			{
				BindingMapPair.Value.RemoveAllSwap(CreateProtocolComparator(MIDIProtocolEntity->GetPropertyId()));
			}
		}
	}
}

void FRemoteControlProtocolMIDI::OnReceiveEvent(UMIDIDeviceInputController* MIDIDeviceController, int32 Timestamp, int32 Type, int32 Channel, int32 MessageData1, int32 MessageData2)
{
	const EMIDIEventType MIDIEventType = static_cast<EMIDIEventType>(Type);

#if WITH_EDITOR
	ProcessAutoBinding(MIDIEventType, Channel, MessageData1);
#endif

#if WITH_EDITOR
	{		
		FRemoteControlLogger::Get().Log(ProtocolName, [DeviceName = MIDIDeviceController->GetDeviceName(), Type, Channel, MessageData1, MessageData2]
		{
			const FString TypeName = StaticEnum<EMIDIEventType>()->GetNameStringByValue(Type);
			return FText::Format(LOCTEXT("MIDIEventLog","Device {0} Type {1} TypeName {2}, Channel {3}, MessageData1 {4}, MessageData2 {5}"), FText::FromString(DeviceName), Type, FText::FromString(TypeName), Channel, MessageData1, MessageData2);
		});
	}
#endif

	if (MIDIEventType == EMIDIEventType::NoteOn || MIDIEventType == EMIDIEventType::NoteOff)
	{
		for (const TPair<FGuid, FRemoteControlProtocolEntityWeakPtr>& MIDIDeviceBindingPair : MIDIDeviceBindings_NoteOn)
		{
			if (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntityPtr = MIDIDeviceBindingPair.Value.Pin())
			{
				const FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = ProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();

				// ignore input if disabled
				if(!MIDIProtocolEntity->IsEnabled())
				{
					continue;
				}

				// ignore if MessageData1 doesn't match
				if(MessageData1 != MIDIProtocolEntity->MessageData1)
				{
					continue;
				}

				// Special case for note on/off: if the current protocol binding uses NoteOn, NoteOff is part of the toggled state
				const bool bIsNoteToggle = MIDIProtocolEntity->EventType == EMIDIEventType::NoteOn && (MIDIEventType == EMIDIEventType::NoteOn || MIDIEventType == EMIDIEventType::NoteOff);
				if(!bIsNoteToggle && (MIDIEventType != MIDIProtocolEntity->EventType || Channel != MIDIProtocolEntity->Channel))
				{
				    continue;
				}

				MessageData2 = MIDIEventType == EMIDIEventType::NoteOn ? 127 : 0;
				QueueValue(ProtocolEntityPtr, MessageData2);
			}
		}
	}
	else if (MIDIEventType == EMIDIEventType::ChannelAfterTouch)
	{
		for (const TPair<FGuid, FRemoteControlProtocolEntityWeakPtr>& MIDIDeviceBindingPair : MIDIDeviceBindings_ChannelAfterTouch)
		{
			if (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntityPtr = MIDIDeviceBindingPair.Value.Pin())
			{
				const FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = ProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();

				// ignore input if disabled
				if(!MIDIProtocolEntity->IsEnabled())
				{
					continue;
				}
				
				if (MIDIEventType != MIDIProtocolEntity->EventType || Channel != MIDIProtocolEntity->Channel)
				{
					continue;
				}

				QueueValue(ProtocolEntityPtr, MessageData1);
			}
		}
	}
	else if (MIDIEventType == EMIDIEventType::ControlChange)
	{
		if (const TMap<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>* MIDIMapBindingsPtr = MIDIDeviceBindings_ControlChange.Find(MIDIDeviceController))
		{
			if (const TArray<FRemoteControlProtocolEntityWeakPtr>* ProtocolEntityArrayPtr = MIDIMapBindingsPtr->Find(MessageData1))
			{
				for (const FRemoteControlProtocolEntityWeakPtr& ProtocolEntityWeakPtr : *ProtocolEntityArrayPtr)
				{
					if (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntityPtr = ProtocolEntityWeakPtr.Pin())
					{
						const FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = ProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();

						// ignore input if disabled
						if(!MIDIProtocolEntity->IsEnabled())
						{
							continue;
						}

						if (MIDIEventType != MIDIProtocolEntity->EventType || Channel != MIDIProtocolEntity->Channel)
						{
							continue;
						}

						QueueValue(ProtocolEntityPtr, MessageData2);
					}
				}
			}
		}	
	}
}

#if WITH_EDITOR
void FRemoteControlProtocolMIDI::ProcessAutoBinding(EMIDIEventType MIDIEventType, int32 Channel, int32 MessageData1)
{
	// Bind only in Editor
	if (!GIsEditor)
	{
		return;
	}
		
	IRemoteControlProtocolWidgetsModule& RCWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
	const TSharedPtr<IRCProtocolBindingList> RCProtocolBindingList = RCWidgetsModule.GetProtocolBindingList();
	if (RCProtocolBindingList.IsValid())
	{
		for (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& ProtocolEntityPtr : RCProtocolBindingList->GetAwaitingProtocolEntities())
		{
			if (ProtocolEntityPtr.IsValid())
			{
				if ((*ProtocolEntityPtr)->GetBindingStatus() == ERCBindingStatus::Awaiting)
				{
					Unbind(ProtocolEntityPtr);
					
					FRemoteControlMIDIProtocolEntity* MIDIProtocolEntity = ProtocolEntityPtr->CastChecked<FRemoteControlMIDIProtocolEntity>();
					MIDIProtocolEntity->EventType = MIDIEventType;
					MIDIProtocolEntity->Channel = Channel;
					MIDIProtocolEntity->MessageData1 = MessageData1;

					Bind(ProtocolEntityPtr);
				}
			}	
		}
	}
}

void FRemoteControlProtocolMIDI::RegisterColumns()
{
	FRemoteControlProtocol::RegisterColumns();

	REGISTER_COLUMN(RemoteControlMIDIProtocolColumns::Channel
		, LOCTEXT("RCPresetChannelColumnHeader", "Channel")
		, ProtocolColumnConstants::ColumnSizeMicro);
	
	REGISTER_COLUMN(RemoteControlMIDIProtocolColumns::Identifier
		, LOCTEXT("RCPresetIdentifierColumnHeader", "ID")
		, ProtocolColumnConstants::ColumnSizeMicro);

	REGISTER_COLUMN(RemoteControlMIDIProtocolColumns::Type
		, LOCTEXT("RCPresetTypeColumnHeader", "Type")
		, ProtocolColumnConstants::ColumnSizeSmall);
}

#endif // WITH_EDITOR

void FRemoteControlProtocolMIDI::UnbindAll()
{
	MIDIDeviceBindings_NoteOn.Empty();
	MIDIDeviceBindings_ControlChange.Empty();
	MIDIDeviceBindings_ChannelAfterTouch.Empty();
	MIDIDevices.Empty();
}

#undef LOCTEXT_NAMESPACE
