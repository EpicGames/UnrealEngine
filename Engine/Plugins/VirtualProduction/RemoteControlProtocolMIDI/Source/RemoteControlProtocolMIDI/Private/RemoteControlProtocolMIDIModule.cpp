// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolMIDIModule.h"

#include "IRemoteControlProtocolModule.h"
#include "MIDIDeviceManager.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolMIDI.h"
#include "RemoteControlProtocolMIDISettings.h"
#include "Async/Async.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

DEFINE_LOG_CATEGORY(LogRemoteControlProtocolMIDI);

#define LOCTEXT_NAMESPACE "FRemoteControlProtocolMIDIModule"

void FRemoteControlProtocolMIDIModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("MIDIDevice"));

#if WITH_EDITOR
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	// Register MIDI Remote Control global settings
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "Remote Control MIDI Protocol",
		                                 LOCTEXT("ProjectSettings_Label", "Remote Control MIDI Protocol"),
		                                 LOCTEXT("ProjectSettings_Description", "Configure MIDI remote control plugin global settings"),
		                                 GetMutableDefault<URemoteControlProtocolMIDISettings>()
		);
	}
#endif // WITH_EDITOR


	const IRemoteControlProtocolModule& RemoteControlProtocolModule = IRemoteControlProtocolModule::Get();
	if (!RemoteControlProtocolModule.IsRCProtocolsDisable())
	{
		IRemoteControlProtocolModule::Get().AddProtocol(FRemoteControlProtocolMIDI::ProtocolName, MakeShared<FRemoteControlProtocolMIDI>());
		GetOnMIDIDevicesUpdated().AddRaw(this, &FRemoteControlProtocolMIDIModule::HandleMIDIDevicesUpdated);
	}
}

void FRemoteControlProtocolMIDIModule::ShutdownModule()
{
	GetOnMIDIDevicesUpdated().RemoveAll(this);

#if WITH_EDITOR
	// Unregister MIDI Remote Control global settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "Remote Control MIDI Protocol");
	}
#endif // WITH_EDITOR
}

void FRemoteControlProtocolMIDIModule::HandleMIDIDevicesUpdated(FMIDIDeviceCollection& InDevices)
{
	static FName ProtocolName = "MIDI";
	const TSharedPtr<IRemoteControlProtocol> RegisteredProtocol = IRemoteControlProtocolModule::Get().GetProtocolByName(ProtocolName);
	check(ProtocolName.IsValid());
	
	// Re-initialize any existing presets
	for(TObjectIterator<URemoteControlPreset> PresetIter; PresetIter; ++PresetIter)
	{
		URemoteControlPreset* Preset = *PresetIter;
		if(Preset != nullptr && IsValidChecked(Preset))
		{
			if(!ensure(IsValid(Preset)))
			{
				UE_LOGF(LogRemoteControlProtocolMIDI, Error, "Attempted to load an invalid Preset: %ls", *Preset->GetFName().ToString());
				return;
			}
		
			// Iterate over each contained property
			for(TWeakPtr<FRemoteControlProperty> ExposedProperty : Preset->GetExposedEntities<FRemoteControlProperty>())
			{
				for(FRemoteControlProtocolBinding& Binding : ExposedProperty.Pin()->ProtocolBindings)
				{
					if(Binding.GetProtocolName() == ProtocolName)
					{
						RegisteredProtocol->Bind(Binding.GetRemoteControlProtocolEntityPtr());
					}
				}
			}
		}
	}
}

TFuture<TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>> FRemoteControlProtocolMIDIModule::GetMIDIDevices(bool bRefresh)
{
	// If not yet initialized, set bRefresh to true
	if(!MIDIDeviceCache.IsValid())
	{
		bRefresh = true;
		MIDIDeviceCache = MakeShared<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>();
	}

	auto Promise = MakeShared<TPromise<TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>>, ESPMode::ThreadSafe>();
	if(bRefresh && !bIsUpdatingDevices)
	{
		bIsUpdatingDevices.store(true);
		// Empty but set size to previous - list is likely to be the same
		MIDIDeviceCache->Empty(MIDIDeviceCache->Num());
		Async(EAsyncExecution::TaskGraph, [&]()
        {
            TArray<FFoundMIDIDevice> Devices;
            UMIDIDeviceManager::FindMIDIDevices(Devices);
            return Devices;
        })
        .Next([this, Promise](TArray<FFoundMIDIDevice> InFoundDevices)
        {
            // Populate the cache
            for(FFoundMIDIDevice& FoundDevice : InFoundDevices)
            {
            	// Only input devices
            	if(FoundDevice.bCanReceiveFrom)
            	{
            		MIDIDeviceCache->Add(MoveTemp(FoundDevice));	 
            	}
            }

        	bIsUpdatingDevices.store(false);

			if(OnMIDIDevicesUpdated.IsBound())
			{
				AsyncTask(ENamedThreads::GameThread, [&]()
				{
					OnMIDIDevicesUpdated.Broadcast(MIDIDeviceCache);	
				});
			}

        	Promise->EmplaceValue(MIDIDeviceCache);
        });
	}
	else
	{
		// If not refreshed, return immediately (can't use MakeFulfilledPromise due to TSharedRef)
		Promise->EmplaceValue(MIDIDeviceCache);
	}

	return Promise->GetFuture();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRemoteControlProtocolMIDIModule, RemoteControlProtocolMIDI);
