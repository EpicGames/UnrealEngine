// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <atomic>


#include "IRemoteControlProtocolMIDIModule.h"

REMOTECONTROLPROTOCOLMIDI_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteControlProtocolMIDI, Log, All);

/**
* MIDI remote control module
*/
class FRemoteControlProtocolMIDIModule : public IRemoteControlProtocolMIDIModule
{
public:
	FRemoteControlProtocolMIDIModule()
		: bIsUpdatingDevices(false)
	{}
	
	//~ Begin IModuleInterface
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	/** Get MIDI devices asynchronously. If bRefresh is true, device info is flushed and re-initialized. */
	virtual TFuture<TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>> GetMIDIDevices(bool bRefresh = false) override;

	/** Checks if MIDI devices are currently being refreshed/updated. */
	virtual bool IsUpdatingDevices() const override { return bIsUpdatingDevices.load(); }

	/** Get MIDI Devices Updated callback. */
	virtual FOnMIDIDevicesUpdated& GetOnMIDIDevicesUpdated() override { return OnMIDIDevicesUpdated; }

private:
	/** Called when MIDI devices are updated, to attempt rebind. */
	void HandleMIDIDevicesUpdated(FMIDIDeviceCollection& InDevices);

private:
	/** Maintain a list of MIDI devices until they're updated. */
	TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe> MIDIDeviceCache;

	/** Flag to indicate if MIDI devices are being updating, to prevent redundant update requests. */
	std::atomic<bool> bIsUpdatingDevices;

	/** Callback to indicate when MIDI devices are updated. */
	FOnMIDIDevicesUpdated OnMIDIDevicesUpdated;
};

