// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolMIDISettings.h"

#include "Modules/ModuleManager.h"
#include "IRemoteControlProtocolMIDIModule.h"

#if WITH_EDITOR
void URemoteControlProtocolMIDISettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{	
	// If DefaultDevice was changed
	if(PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URemoteControlProtocolMIDISettings, DefaultDevice))
	{
		// Re-resolve device id
		IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::LoadModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
		RemoteControlProtocolMIDI.GetMIDIDevices()
			.Next([&](const TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe> InFoundDevices)
			{
				DefaultDevice.ResolveDeviceId(*InFoundDevices);
			});
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void URemoteControlProtocolMIDISettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Resolve device id
	IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::LoadModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
	RemoteControlProtocolMIDI.GetMIDIDevices()
		.Next([&](const TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe> InFoundDevices)
	    {
	        DefaultDevice.ResolveDeviceId(*InFoundDevices);
	    });
}
