// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

struct FFoundMIDIDevice;

using FMIDIDeviceCollection = TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>;

/**
 * Callback called when MIDI devices are updated.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMIDIDevicesUpdated, FMIDIDeviceCollection& /* MIDIDevices */);

/**
  * MIDI remote control module interface
  */
class IRemoteControlProtocolMIDIModule : public IModuleInterface
{
public:
	/** Get MIDI devices asynchronously. If bRefresh is true, device info is flushed and re-initialized. */
	virtual TFuture<TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>> GetMIDIDevices(bool bRefresh = false) = 0;

	/** Checks if MIDI devices are currently being refreshed/updated. */
	virtual bool IsUpdatingDevices() const = 0;

	/** Get MIDI Devices Updated callback. */
	virtual FOnMIDIDevicesUpdated& GetOnMIDIDevicesUpdated() = 0;
};
