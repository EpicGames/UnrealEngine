// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MIDIDeviceController.h"
#include "RemoteControlProtocol.h"
#include "RemoteControlProtocolBinding.h"

#include "UObject/StrongObjectPtr.h"
#include "MIDIDeviceController.h"

#include "RemoteControlProtocolMIDI.generated.h"

struct FFoundMIDIDevice;
class UMIDIDeviceInputController;

/**
 * MIDI protocol device selector type
 */
UENUM()
enum class ERemoteControlMIDIDeviceSelector
{
	ProjectSettings = 0		UMETA(DisplayName = "Use Project Settings", ToolTip = "Uses the Default MIDI Device specified in Project Settings."),
	DeviceName = 1			UMETA(DisplayName = "Device Name", ToolTip = "User-specified device name."),
	DeviceId = 2			UMETA(DisplayName = "Device Id", ToolTip = "User-specified device id.")
};

/**
 * MIDI protocol device identifier
 */
USTRUCT()
struct REMOTECONTROLPROTOCOLMIDI_API FRemoteControlMIDIDevice
{
	GENERATED_BODY()

public:
	/** Default constructor */
	FRemoteControlMIDIDevice() = default;

	/** Construct for the given DeviceId and DeviceName */
	FRemoteControlMIDIDevice(const int32 DeviceId, const FName& DeviceName)
        : DeviceSelector(ERemoteControlMIDIDeviceSelector::ProjectSettings)
		, ResolvedDeviceId(DeviceId)
        , DeviceName(DeviceName)
	{
	}

	/** Construct for the given DeviceId */
	FRemoteControlMIDIDevice(const int32 DeviceId)
		: DeviceSelector(ERemoteControlMIDIDeviceSelector::DeviceId)
		, ResolvedDeviceId(DeviceId)
		, DeviceId(DeviceId)
	{
	}

	/** Midi Device Selector */
	UPROPERTY(EditAnywhere, Category = Mapping)
	ERemoteControlMIDIDeviceSelector DeviceSelector = ERemoteControlMIDIDeviceSelector::ProjectSettings;
	
	/** Midi Resolved Device Id. Distinct from the user specified Device Id. */
	UPROPERTY(VisibleAnywhere, Category = Mapping, meta = (ClampMin = 0, UIMin = 0))
	int32 ResolvedDeviceId = -1;

	/** Midi Device Name. If specified, takes priority over DeviceId. */
	UPROPERTY(EditAnywhere, Category = Mapping)
	FName DeviceName = NAME_None;

	/** User-specified Midi Device Id */
	UPROPERTY(EditAnywhere, Category = Mapping, meta = (ClampMin = 0, UIMin = 0))
	int32 DeviceId = 1;

	/** If device available for use. */
	UPROPERTY(EditAnywhere, Category = Mapping)
	bool bDeviceIsAvailable = false;

	/** Resolves the actual Midi Device Id given the FRemoteControlMIDIDevice configuration. Returns INDEX_NONE if device not found or invalid. */
	int32 ResolveDeviceId(const TArray<FFoundMIDIDevice>& InFoundDevices);

	/** Sets DeviceId and DeviceName, disables bUseProjectSettings and bUseUserDeviceId. */
	void SetDevice(const int32 InDeviceId, const FName& InDeviceName);

	/** Sets bUseProjectSettings = true, clears bUseUserDeviceId. */
	void SetUseProjectSettings();

	/** Sets bUseUserDeviceId = true, clears bUseProjectSettings. */
	void SetUserDeviceId();

	/** Creates a formatted string to display in the combobox */
	FText ToDisplayName() const;
};

/** FRemoteControlMIDIDevice equality, depending on what properties are specified Comparison  */
inline uint64 GetTypeHash(const FRemoteControlMIDIDevice& InValue)
{
	uint64 BaseHash;
	switch (InValue.DeviceSelector)
	{
		default:
		case ERemoteControlMIDIDeviceSelector::ProjectSettings:
			BaseHash = GetTypeHash(InValue.ResolvedDeviceId);
			break;
		
		case ERemoteControlMIDIDeviceSelector::DeviceName:
			BaseHash = GetTypeHash(InValue.DeviceName);
			break;

		case ERemoteControlMIDIDeviceSelector::DeviceId:
			BaseHash = GetTypeHash(InValue.DeviceId);
			break;
	}
	return HashCombine(BaseHash, GetTypeHash(InValue.DeviceSelector));
}

inline bool operator==(const FRemoteControlMIDIDevice& Lhs, const FRemoteControlMIDIDevice& Rhs) { return GetTypeHash(Lhs) == GetTypeHash(Rhs); }
inline bool operator!=(const FRemoteControlMIDIDevice& Lhs, const FRemoteControlMIDIDevice& Rhs) { return GetTypeHash(Lhs) != GetTypeHash(Rhs); }

/**
 * MIDI protocol entity for remote control binding
 */
USTRUCT()
struct FRemoteControlMIDIProtocolEntity : public FRemoteControlProtocolEntity
{
	GENERATED_BODY()

public:
	//~ Begin FRemoteControlProtocolEntity interface
	virtual FName GetRangePropertyName() const override { return NAME_ByteProperty; }
	//~ End FRemoteControlProtocolEntity interface

public:
	/** Midi Device */
	UPROPERTY(EditAnywhere, Category = Mapping, AdvancedDisplay)
	FRemoteControlMIDIDevice Device;

	/** Midi Event type */
	UPROPERTY(EditAnywhere, Category = Mapping)
	EMIDIEventType EventType = EMIDIEventType::ControlChange;

	/** Midi button event message data id for binding */
	UPROPERTY(EditAnywhere, Category = Mapping, meta = (DisplayName = "Mapped channel Id"))
	int32 MessageData1 = 0;

	/** Midi device channel */
	UPROPERTY(EditAnywhere, Category = Mapping)
	int32 Channel = 1;

	/** Midi range input property template, used for binding. */
	UPROPERTY(Transient, meta = (ClampMin = 0, ClampMax = 127))
	uint8 RangeInputTemplate = 0;

	/**
	* Checks if this entity has the same values as the Other.
	* Used to check for duplicate inputs.
	*/
	virtual bool IsSame(const FRemoteControlProtocolEntity* InOther) override;

#if WITH_EDITOR

	/** Register(s) all the widgets of this protocol entity. */
	virtual void RegisterProperties() override;

#endif // WITH_EDITOR
};

/**
 * MIDI protocol implementation for Remote Control
 */
class FRemoteControlProtocolMIDI : public FRemoteControlProtocol
{
public:
	FRemoteControlProtocolMIDI()
		: FRemoteControlProtocol(ProtocolName)
	{}
	
	//~ Begin IRemoteControlProtocol interface
	virtual void Bind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr) override;
	virtual void Unbind(FRemoteControlProtocolEntityPtr InRemoteControlProtocolEntityPtr) override;
	virtual void UnbindAll() override;
	virtual UScriptStruct* GetProtocolScriptStruct() const override { return FRemoteControlMIDIProtocolEntity::StaticStruct(); }
	//~ End IRemoteControlProtocol interface

private:
	/**
	 * On receive MIDI buffer callback
	 * @param	MIDIDeviceController	The MIDI Input Device Controller
	 * @param	Timestamp				The MIDI timestamp
	 * @param	Channel					The MIDI channel to send 
	 * @param	MessageData1			The first part of the MIDI data
	 * @param	MessageData2			The second part of the MIDI data
	 */
	void OnReceiveEvent(UMIDIDeviceInputController* MIDIDeviceController, int32 Timestamp, int32 Type, int32 Channel, int32 MessageData1, int32 MessageData2);

#if WITH_EDITOR
	/**
	 * Process the AutoBinding to the Remote Control Entity
	 * @param	MIDIEventType		The event type as specified in the EMIDIEventType struct
	 * @param	Channel				The MIDI channel 
	 * @param	MessageData1		The first part of the MIDI data
	 */
	void ProcessAutoBinding(EMIDIEventType MIDIEventType, int32 Channel, int32 MessageData1);

protected:

	/** Populates protocol specific columns. */
	virtual void RegisterColumns() override;
#endif // WITH_EDITOR

private:

	/** Binding for ControlChange (11) MIDI protocol */
	TMap<UMIDIDeviceInputController*, TMap<int32, TArray<FRemoteControlProtocolEntityWeakPtr>>> MIDIDeviceBindings_ControlChange;

	/** Binding for NoteOn (9) MIDI protocol */
	TMap<FGuid, FRemoteControlProtocolEntityWeakPtr> MIDIDeviceBindings_NoteOn;

	/** Binding for ChannelAfterTouch (13) MIDI protocol */
	TMap<FGuid, FRemoteControlProtocolEntityWeakPtr> MIDIDeviceBindings_ChannelAfterTouch;

	/** MIDI devices */
	TMap<int32, TStrongObjectPtr<UMIDIDeviceInputController>> MIDIDevices;

public:
	/** MIDI protocol name */
	static const FName ProtocolName;
};
