// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RemoteControlProtocolMIDI.h"
#include "UObject/Object.h"

#include "RemoteControlProtocolMIDISettings.generated.h"

/**
 * MIDI Remote Control Settings
 */
UCLASS(Config = Engine, DefaultConfig)
class REMOTECONTROLPROTOCOLMIDI_API URemoteControlProtocolMIDISettings : public UObject
{
	GENERATED_BODY()

public:
	URemoteControlProtocolMIDISettings()
	{
		// The DeviceSelector shouldn't be the project settings - this IS the project settings!
		DefaultDevice.DeviceSelector = ERemoteControlMIDIDeviceSelector::DeviceId;
	}
	
	/** Midi Default Device */
	UPROPERTY(Config, EditAnywhere, Category = MIDI)
	FRemoteControlMIDIDevice DefaultDevice = {1};

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	virtual void PostInitProperties() override;
};
