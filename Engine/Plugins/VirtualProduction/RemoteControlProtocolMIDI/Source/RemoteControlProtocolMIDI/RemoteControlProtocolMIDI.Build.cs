// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlProtocolMIDI : ModuleRules
{
	public RemoteControlProtocolMIDI(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"MIDIDevice",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RemoteControl",
				"RemoteControlProtocol",
			}
		);
		
		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"InputCore",
					"RemoteControlProtocolWidgets",
				}
			);
		}
	}
}
