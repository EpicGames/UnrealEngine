// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlMIDIDeviceCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorFontGlyphs.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "IRemoteControlProtocolMIDIModule.h"
#include "ISinglePropertyView.h"
#include "MIDIDeviceManager.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "RemoteControlProtocolMIDI.h"
#include "RemoteControlProtocolMIDISettings.h"
#include "Algo/MaxElement.h"
#include "Algo/MinElement.h"
#include "Algo/Transform.h"
#include "Async/Async.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "FRemoteControlMIDIDeviceCustomization"

FRemoteControlMIDIDeviceCustomization::FRemoteControlMIDIDeviceCustomization()
{
	IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::LoadModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
	RemoteControlProtocolMIDI.GetOnMIDIDevicesUpdated().AddRaw(this, &FRemoteControlMIDIDeviceCustomization::OnMIDIDevicesUpdated);
}

FRemoteControlMIDIDeviceCustomization::~FRemoteControlMIDIDeviceCustomization()
{
	if(FModuleManager::Get().IsModuleLoaded("RemoteControlProtocolMIDI"))
	{
		IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::LoadModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
		RemoteControlProtocolMIDI.GetOnMIDIDevicesUpdated().RemoveAll(this);
	}
}

void FRemoteControlMIDIDeviceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	const TSharedPtr<IPropertyUtilities> PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	bool bIsProjectSettings = false;
	if(OuterObjects.Num() > 0 && OuterObjects[0]->GetClass() == URemoteControlProtocolMIDISettings::StaticClass())
	{
		bIsProjectSettings = true;
	}
	
	check(CastFieldChecked<FStructProperty>(PropertyHandle->GetProperty())->Struct == FRemoteControlMIDIDevice::StaticStruct());

	const TSharedPtr<IPropertyHandle> DeviceSelectorHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceSelector));
	if(bIsProjectSettings)
	{
		UEnum* DeviceSelectorEnum = StaticEnum<ERemoteControlMIDIDeviceSelector>();
		static FText RestrictReason = LOCTEXT("IsProjectSettings", "Project Settings can't inherit itself");
		TSharedPtr<FPropertyRestriction> EnumRestriction = MakeShared<FPropertyRestriction>(RestrictReason);
		EnumRestriction->AddHiddenValue(DeviceSelectorEnum->GetNameStringByIndex(0));
		DeviceSelectorHandle->AddRestriction(EnumRestriction.ToSharedRef());
	}
 
	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.0f)
	.MaxDesiredWidth(0.0f)
	[
		DeviceSelectorHandle->CreatePropertyValueWidget()
	]
	.IsEnabled(MakeAttributeLambda([=]{ return !DeviceSelectorHandle->IsEditConst() && PropertyUtilities->IsPropertyEditingEnabled(); }));
}

void FRemoteControlMIDIDeviceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> DeviceSelectorHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceSelector));

	// Use Project Device
	{
		TSharedPtr<SWidget> ProjectDeviceNameWidget;
		TSharedPtr<SWidget> ProjectDeviceValueWidget;
		GetProjectDeviceWidgets(ProjectDeviceNameWidget, ProjectDeviceValueWidget);
		
		FDetailWidgetRow& ProjectDeviceRow = InChildBuilder.AddCustomRow(LOCTEXT("UseProjectDeviceRow", "Project Device"));
		ProjectDeviceRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([DeviceSelectorHandle]()
        {
            uint8 DeviceSelectorByte = 0;
            DeviceSelectorHandle->GetValue(DeviceSelectorByte);
            const ERemoteControlMIDIDeviceSelector DeviceSelector = static_cast<ERemoteControlMIDIDeviceSelector>(DeviceSelectorByte);
		
            return DeviceSelector == ERemoteControlMIDIDeviceSelector::ProjectSettings ? EVisibility::Visible : EVisibility::Collapsed;
        })))
		.NameContent()
		[
			ProjectDeviceNameWidget.ToSharedRef()
		]
        .ValueContent()
        [
			// Displays read-only info about project device
        	ProjectDeviceValueWidget.ToSharedRef()
        ];
	}

	// User specified Device Name
	{
		using FMIDIDevicePtr = TSharedPtr<FMIDIDeviceItem, ESPMode::ThreadSafe>;
		using SDeviceNameListType = SListView<FMIDIDevicePtr>;

		const TSharedPtr<IPropertyHandle> DeviceNameHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceName));
		IDetailPropertyRow& DeviceNameRow = InChildBuilder.AddProperty(DeviceNameHandle.ToSharedRef());
		DeviceNameRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([DeviceSelectorHandle]()
		{
			uint8 DeviceSelectorByte = 0;
			DeviceSelectorHandle->GetValue(DeviceSelectorByte);
			const ERemoteControlMIDIDeviceSelector DeviceSelector = static_cast<ERemoteControlMIDIDeviceSelector>(DeviceSelectorByte);
			
			return DeviceSelector == ERemoteControlMIDIDeviceSelector::DeviceName ? EVisibility::Visible : EVisibility::Collapsed;
		})))
		.ShouldAutoExpand(true)
		.CustomWidget()
		.NameContent()
		[
			DeviceNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			// Device name, with auto-discovery of device names. Allows entry of non-existent device (may not be on this particular users system)
			SAssignNew(DeviceNameComboButton, SComboButton)
		    .ContentPadding(FMargin(0,0,5,0))
		    .CollapseMenuOnParentFocus(true)
		    .ButtonContent()
		    [
		        SNew(SBorder)
		        .BorderImage( FAppStyle::GetBrush("NoBorder") )
		        .Padding(FMargin(0, 0, 5, 0))
		        [
		            SNew(SEditableTextBox)
					.Text_Lambda([DeviceNameHandle]()
					{
						FName DeviceName;
						DeviceNameHandle->GetValue(DeviceName);
						return FText::FromName(DeviceName);
					})
					.OnTextCommitted_Lambda([DeviceNameHandle](const FText& InNewText, ETextCommit::Type InTextCommit)
					{
						DeviceNameHandle->SetValue(FName(InNewText.ToString()));
					})
	                .SelectAllTextWhenFocused(true)
	                .RevertTextOnEscape(true)
	                .Font(IDetailLayoutBuilder::GetDetailFont())
		        ]
		    ]
		    .OnGetMenuContent_Lambda([this, DeviceNameHandle]()
		    {
				return SNew(SVerticalBox)
	                +SVerticalBox::Slot()
	                .AutoHeight()
	                .MaxHeight(400.0f)
	                [
	                	SNew(SWidgetSwitcher)
	                	.WidgetIndex_Lambda([&]
	                	{
	                		return DeviceSource.Num() > 0 ? 0 : 1;
	                	})
	                	+ SWidgetSwitcher::Slot()
	                	[
	                		SAssignNew(DeviceNameListView, SDeviceNameListType)
	                        .ListItemsSource(&DeviceSource)
	                        .OnGenerateRow_Lambda([](FMIDIDevicePtr InItem, const TSharedRef<STableViewBase>& InOwnerTable)
	                        {
	                            return SNew(STableRow<FMIDIDevicePtr>, InOwnerTable)
	                            [
	                                SNew(STextBlock).Text(InItem->DeviceName)
	                                    .IsEnabled_Lambda([InItem] { return InItem->bIsAvailable; })
	                            ];
	                        })
	                        .OnSelectionChanged_Lambda([this, DeviceNameHandle](FMIDIDevicePtr InProposedSelection, ESelectInfo::Type InSelectInfo)
	                        {
	                            if(InProposedSelection.IsValid())
	                            {
	                                DeviceNameHandle->SetValue(FName(*InProposedSelection->DeviceName.ToString()));

	                                const TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(DeviceNameListView.ToSharedRef()).ToSharedRef();
	                                FSlateApplication::Get().RequestDestroyWindow( ParentContextMenuWindow );
	                            }
	                        })
	                	]

	                	+ SWidgetSwitcher::Slot()
	                	[
	                		SNew(STextBlock).Text(LOCTEXT("NoDevices", "No devices found"))
	                	]
	                ];
		    })
		];
	}

	// User specified Device Id
	{
		const TSharedPtr<IPropertyHandle> DeviceIdHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceId));

		// Set min and max id's for UI only (still allow manual entry or persistence of out-of-current-range id's)
		{
			TArray<int32> DeviceIds;
			Algo::Transform(DeviceSource, DeviceIds, [](const TSharedPtr<FMIDIDeviceItem, ESPMode::ThreadSafe>& InDevice)
            {
                return InDevice->DeviceId;
            });

			int32* MinDeviceId = Algo::MinElement(DeviceIds);
			int32* MaxDeviceId = Algo::MaxElement(DeviceIds);

			if(MinDeviceId)
			{
				DeviceIdHandle->SetInstanceMetaData(TEXT("UIMin"), FString::FormatAsNumber((*MinDeviceId)));
			}

			DeviceIdHandle->SetInstanceMetaData(TEXT("UIMax"), FString::FormatAsNumber(MaxDeviceId ? *MaxDeviceId : 1));
		}

		IDetailPropertyRow& DeviceIdRow = InChildBuilder.AddProperty(DeviceIdHandle.ToSharedRef());
		DeviceIdRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([DeviceSelectorHandle]()
        {
            uint8 DeviceSelectorByte = 0;
            DeviceSelectorHandle->GetValue(DeviceSelectorByte);
            const ERemoteControlMIDIDeviceSelector DeviceSelector = static_cast<ERemoteControlMIDIDeviceSelector>(DeviceSelectorByte);
		
            return DeviceSelector == ERemoteControlMIDIDeviceSelector::DeviceId ? EVisibility::Visible : EVisibility::Collapsed;
        })))
        .CustomWidget()
        .NameContent()
        [
            DeviceIdHandle->CreatePropertyNameWidget()
        ]
        .ValueContent()
        [
            DeviceIdHandle->CreatePropertyValueWidget()
        ];
	}

	// Refresh MIDI Devices
	{
		FDetailWidgetRow& RefreshDevicesRow = InChildBuilder.AddCustomRow(LOCTEXT("RefreshDevicesRow", "Refresh Devices"));
		RefreshDevicesRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([DeviceSelectorHandle]()
        {
            uint8 DeviceSelectorByte = 0;
            DeviceSelectorHandle->GetValue(DeviceSelectorByte);
            const ERemoteControlMIDIDeviceSelector DeviceSelector = static_cast<ERemoteControlMIDIDeviceSelector>(DeviceSelectorByte);
		
            return DeviceSelector != ERemoteControlMIDIDeviceSelector::ProjectSettings ? EVisibility::Visible : EVisibility::Collapsed;
        })))
        .ValueContent()
        [
			MakeRefreshButton()
        ];
	}
	
	IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::GetModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
	// @note: There's a chance this will end firing twice (callback + return value), but the only side effect is the perf overhead
	RemoteControlProtocolMIDI.GetMIDIDevices().Next([&](auto InMIDIDevices) { OnMIDIDevicesUpdated(InMIDIDevices); });
}

TSharedRef<SWidget> FRemoteControlMIDIDeviceCustomization::MakeRefreshButton() const
{
	return SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.Padding(0.0f, 0.0f, .0f, 0.0f)
	[
		SNew(SButton)
	    .ForegroundColor(FSlateColor::UseForeground())
	    .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Default"))
	    .IsEnabled_Lambda([]()
	    {
	    	IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::GetModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
	    	return !RemoteControlProtocolMIDI.IsUpdatingDevices();
	    })
	    .HAlign(HAlign_Center)
	    .VAlign(VAlign_Center)
	    .OnClicked_Lambda([]()
	    {
	    	IRemoteControlProtocolMIDIModule& RemoteControlProtocolMIDI = FModuleManager::GetModuleChecked<IRemoteControlProtocolMIDIModule>("RemoteControlProtocolMIDI");
	    	RemoteControlProtocolMIDI.GetMIDIDevices(true);
	    	return FReply::Handled();
	    })
	    .ToolTipText(LOCTEXT("BuildClustersAndMeshesToolTip", "Re-generates clusters and then proxy meshes for each of the generated clusters in the level. This dirties the level."))
	    [
	        SNew(SHorizontalBox)

	        // icon
	        + SHorizontalBox::Slot()
	        .AutoWidth()
	        .VAlign(VAlign_Center)
	        [
	            SNew(STextBlock)
	            .TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
	            .Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
	            .Text(FEditorFontGlyphs::Recycle)
	        ]

	        // text
	        + SHorizontalBox::Slot()
	        .AutoWidth()
	        .VAlign(VAlign_Bottom)
	        .Padding(4, 0, 0, 0)
	        [
	            SNew(STextBlock)
	            .TextStyle( FAppStyle::Get(), "ContentBrowser.TopBar.Font" )
	            .Text(LOCTEXT("RefreshDevices", "Refresh"))
	        ]
	    ]
	]
	// matches DetailPropertyRow.cpp:869, adds spacer to the right where reset to defaults would be
	+ SHorizontalBox::Slot()
	.Padding(4.0f, 0.0f)
	.AutoWidth()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Left)
	[
		SNew(SSpacer)
		.Size(FVector2D(8.0f, 8.0f))
	];
}

void FRemoteControlMIDIDeviceCustomization::OnMIDIDevicesUpdated(TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>& InMIDIDevices)
{
	// Empty but set size to previous - list is likely to be the same
	DeviceSource.Empty(DeviceSource.Num());
	
	// (Re)populate the local device list
	for(const FFoundMIDIDevice& FoundDevice : *InMIDIDevices)
	{
		DeviceSource.Add(MakeShared<FMIDIDeviceItem, ESPMode::ThreadSafe>(FoundDevice.DeviceID, FText::FromString(FoundDevice.DeviceName), !FoundDevice.bIsAlreadyInUse));
	}
}

void FRemoteControlMIDIDeviceCustomization::GetProjectDeviceWidgets(
	TSharedPtr<SWidget>& OutNameWidget,
	TSharedPtr<SWidget>& OutValueWidget)
{
	if(!CachedProjectDevicePropertyView.IsValid())
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		const TSharedPtr<ISinglePropertyView> PropertyView = PropertyEditorModule.CreateSingleProperty(
            GetMutableDefault<URemoteControlProtocolMIDISettings>(),
            GET_MEMBER_NAME_CHECKED(URemoteControlProtocolMIDISettings, DefaultDevice), {});
		CachedProjectDevicePropertyView = PropertyView;
	}

	const TSharedPtr<ISinglePropertyView> PropertyView = CachedProjectDevicePropertyView;
	const TSharedPtr<IPropertyHandle> ProjectDeviceHandle = PropertyView->GetPropertyHandle();
	TSharedPtr<IPropertyHandle> DeviceSelectorHandle = ProjectDeviceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceSelector));
	const TSharedPtr<IPropertyHandle> DeviceNameHandle = ProjectDeviceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceName));
	const TSharedPtr<IPropertyHandle> DeviceIdHandle = ProjectDeviceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceId));

	auto GetSwitcherIndex = [this]()
	{
		const TSharedPtr<ISinglePropertyView> PropertyView = CachedProjectDevicePropertyView;
		const TSharedPtr<IPropertyHandle> ProjectDeviceHandle = PropertyView->GetPropertyHandle();
		const TSharedPtr<IPropertyHandle> DeviceSelectorHandle = ProjectDeviceHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRemoteControlMIDIDevice, DeviceSelector));
		
		uint8 DeviceSelectorByte = 0;
		DeviceSelectorHandle->GetValue(DeviceSelectorByte);
		const ERemoteControlMIDIDeviceSelector DeviceSelector = static_cast<ERemoteControlMIDIDeviceSelector>(DeviceSelectorByte);

		return static_cast<uint8>(DeviceSelector) - 1; // -1 to remove UseProjectSetting
	};

	OutNameWidget = SNew(SWidgetSwitcher)
	.WidgetIndex_Lambda(GetSwitcherIndex)
    + SWidgetSwitcher::Slot() 
    [
        DeviceNameHandle->CreatePropertyNameWidget()
    ]
    + SWidgetSwitcher::Slot()
    [
        DeviceIdHandle->CreatePropertyNameWidget()
    ];

	// Get and disable widget - it's for display only
	TSharedRef<SWidget> ProjectDeviceNameValueWidget = DeviceNameHandle->CreatePropertyValueWidget();
	ProjectDeviceNameValueWidget->SetEnabled(false);

	TSharedRef<SWidget> ProjectDeviceIdValueWidget = DeviceIdHandle->CreatePropertyValueWidget();
	ProjectDeviceIdValueWidget->SetEnabled(false);

	OutValueWidget = SNew(SWidgetSwitcher)
    .WidgetIndex_Lambda(GetSwitcherIndex)
    + SWidgetSwitcher::Slot() 
    [
		ProjectDeviceNameValueWidget
    ]
    + SWidgetSwitcher::Slot()
    [
		ProjectDeviceIdValueWidget
    ];
}

#undef LOCTEXT_NAMESPACE
