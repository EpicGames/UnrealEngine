// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPropertyTypeCustomization.h"
#include "MIDIDeviceManager.h"

class ISinglePropertyView;
class ITableRow;
class SComboButton;
class SMIDIDeviceComboBox;
class STableViewBase;
class SWidget;
template <typename ItemType> class SListView;

/** Menu item for a MIDI Device */
struct FMIDIDeviceItem
{
	//FMIDIDeviceItem() = default;
	
	FMIDIDeviceItem(const int32 InDeviceId, const FText& InDeviceName, const bool bInIsAvailable)
		: DeviceId(InDeviceId)
		, DeviceName(InDeviceName)
		, bIsAvailable(bInIsAvailable)
	{
	}

	/** The unique ID of this MIDI device */
	int32 DeviceId;

	/** The name of this device.  This name comes from the MIDI hardware, any might not be unique */
	FText DeviceName;

	/** Whether the device is available */
	bool bIsAvailable;
};

/** Property customization for RemoteControlMIDIDevice */
class REMOTECONTROLPROTOCOLMIDIEDITOR_API FRemoteControlMIDIDeviceCustomization : public IPropertyTypeCustomization
{
public:
	FRemoteControlMIDIDeviceCustomization();
	~FRemoteControlMIDIDeviceCustomization();
	
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FRemoteControlMIDIDeviceCustomization>();
	}
	
	//~ IPropertyTypeCustomization interface begin
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Creates a button to refresh MIDI devices */
	TSharedRef<SWidget> MakeRefreshButton() const;
	
	/** Called when MIDI devices are updated */
	void OnMIDIDevicesUpdated(TSharedPtr<TArray<FFoundMIDIDevice>, ESPMode::ThreadSafe>& InMIDIDevices);
	
	/** A list of all devices to choose from */
	TArray<TSharedPtr<FMIDIDeviceItem, ESPMode::ThreadSafe>> DeviceSource;

	/** Widgets for the device names */
	TSharedPtr<SComboButton> DeviceNameComboButton;
	TSharedPtr<SListView<TSharedPtr<FMIDIDeviceItem, ESPMode::ThreadSafe>>> DeviceNameListView;

	/** Get widgets for project device */
	void GetProjectDeviceWidgets(TSharedPtr<SWidget>& OutNameWidget, TSharedPtr<SWidget>& OutValueWidget);

private:
	/** Root property handle */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Cached Project Device property view */
	TSharedPtr<ISinglePropertyView> CachedProjectDevicePropertyView;
};
