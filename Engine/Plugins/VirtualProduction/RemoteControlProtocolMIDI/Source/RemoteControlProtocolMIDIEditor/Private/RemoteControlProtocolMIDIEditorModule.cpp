// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyEditorModule.h"
#include "RemoteControlProtocolMIDI.h"
#include "RemoteControlProtocolMIDISettings.h"
#include "DetailCustomizations/RemoteControlMIDIDeviceCustomization.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FRemoteControlProtocolMIDIEditorModule"

/**
 * MIDI remote control editor module
 */
class FRemoteControlProtocolMIDIEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomPropertyTypeLayout(FRemoteControlMIDIDevice::StaticStruct()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRemoteControlMIDIDeviceCustomization::MakeInstance));
	}

	virtual void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FRemoteControlMIDIDevice::StaticStruct()->GetFName());
			PropertyEditorModule.NotifyCustomizationModuleChanged();
		}
	}
	//~ End IModuleInterface
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRemoteControlProtocolMIDIEditorModule, RemoteControlProtocolMIDIEditor);
