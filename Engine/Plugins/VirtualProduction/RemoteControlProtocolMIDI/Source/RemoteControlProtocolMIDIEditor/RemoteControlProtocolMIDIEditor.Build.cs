// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RemoteControlProtocolMIDIEditor : ModuleRules
{
	public RemoteControlProtocolMIDIEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"MIDIDevice",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"EditorStyle",
				"EditorWidgets",
				"InputCore",
				"PropertyEditor",
				"RemoteControl",
				"RemoteControlProtocol",
				"RemoteControlProtocolMIDI",
				"Slate",
				"SlateCore",
			}
		);

		ShortName = "RCPMidiEd";
	}
}
